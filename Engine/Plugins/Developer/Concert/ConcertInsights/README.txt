========== Summary ==========
ConcertInsights is a toolchain for tracing steps accross multiple machines in distributed protocols.
- This adds a status bar entry in the bottom right of the editor to start a synchronized trace (ConcertInsightsCore implements shared logic)
- Each machine will generate its own .utrace file (ConcertInsightsClient and ConcertInsightsServer listen for synchronized trace requests)
- ConcertInsightsVisualizer aggregated multiple .utrace files and correlates the machine CPU times

This toolchain is highly experimental. Issues:
- Time correlation is too imprecise with a fault range of 400 ms. 
  This is because the FDateTime::UtcNow seems to be accurate only within about 400ms accross machines.
  To fix this, we should consider a proper time synchronization protocol, e.g. PTP.
 
========== Set-up ==========
To try this out, the minimum set-up is:
- In your .uproject file, to Plugins section add 
	{
		"Name": "MultiUserClient",
		"Enabled": true
	},
	{
		"Name": "ConcertInsightsClient",
		"Enabled": true
	},
	{
		"Name": "ConcertInsightsVisualizer",
		"Enabled": true
	}
- Either unshelf 33391924, or recreate the work in that CL:
	- Engine/Programs/UnrealMultiUserSlateServerConfig/DefaultEngine.ini: Add +ProgramEnabledPlugins=ConcertInsightsServer
	- Engine/Programs/UnrealInsights/DefaultEngine.ini: Add +ProgramEnabledPlugins=ConcertInsightsVisualizer
- Launch Insights with the console variable "Insights.Concert.EnableGameThreadAggregation = true".
	
========== Tutorial with VCam ==========
To view example tracing, do the following:
1. Enable Virtual Camera plugin
2. Add a VCam actor to your level
3. Launch MU server
4. Join machine A to MU session
5. Join machine B to MU session
6. Connect your mobile device to the VCam on machine A (see https://dev.epicgames.com/community/learning/tutorials/aEeW/unreal-engine-vcam-actor-quick-start-using-pixel-streaming)
7. On machine A, in the MU session tab, click Replication, then use the Add button to add the VCamActor for replication (the actor should be automatically set up)
8. On machine B, pilot the VCamActor manually (right-click in outliner > pilot and in viewport toggle piloting on in top-left corner)
9. Notice that moving your mobile device should now move the viewport on both machine A (source) and B (replicated)
8. On machine A or B, in the bottom-right, click Multi User > Start synchronized trace
9. Move your mobile device
10. On machine A or B, click Multi User > Stop synchronized trace
11. Open Unreal Insights
12. Open any of the 3 trace files that were just generated
13. Disable all channels except for "Concert"
