/Binaries/*/*.pdb
/Binaries/*/*Debug*
/Intermediate