## Ignore output
Binaries

##  Ignore backup files 
*~

## Ignore project files in the root
/*.sln
*.suo
*.opensdf
*.sdf
/*.xcodeproj
/Makefile
/CMakeLists.txt
/.idea/**

## Ignore IDE project generated files
*.user
