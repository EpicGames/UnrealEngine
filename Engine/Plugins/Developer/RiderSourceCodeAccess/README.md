# Description
RiderSourceCodeAccess is a plugin for Unreal Engine, available through [Marketplace page](https://www.unrealengine.com/marketplace/en-US/product/rider-source-code-access) that will add an option to select ["Rider for Unreal Engine"](https://www.jetbrains.com/rider/unreal/) as your source code editor in Unreal Editor.

## Functionality
* Adds the option to select "Rider for Unreal Engine" as your IDE of choice in Unreal Editor.
  * You can select a specific version of Rider (e.g. Rider xxx.xx.xx) or,
  * Use the latest version of Rider that’s available on your workstation by selecting `Rider` option or,
  * Use experimental version of .uproject based project model in Rider that is available starting with UE 4.26.1 and Rider for Unreal Engine 2021.1.
* `RiderSourceCodeAccess` looks for `Rider`'s in the next places:
  * Under `JetBrains Toolbox` installation folder;
  * Windows only: `Rider`'s registered in Windows Registry;
  * MacOS and Linux: using `mdfind` utility with `"kMDItemKind == Application"` pattern;
  * Manually installed `Rider`'s. You can specify path to manually installed `Rider` in "RiderSourceCodeAccess\Resources\RiderLocations.txt" file.

![Example of dropdown box with Rider for Unreal Engine](https://user-images.githubusercontent.com/1694911/115036768-76e76c00-9ed6-11eb-8ca5-d457b6051945.png)

# How to add this plugin manually (not recommened)
## How to add to Game project
1. Get the source code of the [master branch](https://github.com/JetBrains/RiderSourceCodeAccess/archive/refs/heads/master.zip); 
2. Unzip `RiderSourceCodeAccess.zip` to `RiderSourceCodeAccess` folder;
3. Copy `RiderSourceCodeAccess` folder to `{GameProjectRoot}/Plugins/Developer`;
  a. If `{GameProjectRoot}/Plugins/Developer` does not exist, create it;
4. (For Windows and .sln project model only) Re-generate solution files - uproject file > context menu > Generate Visual Studio project files;
5. Build UE project using Rider (or any other IDE that you've setup for your OS);
6. Start Unreal Editor;
7. Edit > Editor Preferences ... > General > Source Code > Source Code Editor;
8. Select Rider from drop down list. NB: only Rider with C++ plugins will be available from drop down list.

## How to add to Engine project
### Unreal Engine is built from source code   
1. Get the source code of the [master branch](https://github.com/JetBrains/RiderSourceCodeAccess/archive/refs/heads/master.zip); 
2. Unzip `RiderSourceCodeAccess.zip` to `RiderSourceCodeAccess` folder;
3. Copy `RiderSourceCodeAccess` folder to `{UnrealEngineRoot}/Engine/Plugins/Developer`;
  a. If `{UnrealEngineRoot}/Engine/Plugins/Developer` does not exist, create it;
4. (For Windows and .sln project model only) Re-generate solution files - uproject file > context menu > Generate Visual Studio project files;
5. Build UE project using Rider (or any other IDE that you've setup for your OS);
6. Start Unreal Editor;
7. Edit > Editor Preferences ... > General > Source Code > Source Code Editor;
8. Select Rider from drop down list. NB: only Rider with C++ plugins will be available from drop down list.

### Unreal Engine is downloaded from Epic Games Store
1. Get the source code of the [master branch](https://github.com/JetBrains/RiderSourceCodeAccess/archive/refs/heads/master.zip); 
2. Unzip `RiderSourceCodeAccess.zip` to `RiderSourceCodeAccess` folder;
3. Copy `RiderSourceCodeAccess` folder to `{UnrealProjectRoot}/Plugins/Developer`;
  a. If `{UnrealProjectRoot}/Plugins/Developer` does not exist, create it;
4.(For Windows and .sln project model only) Re-generate solution files - uproject file > context menu > Generate Visual Studio project files;
5. Build UE project using Rider (or any other IDE that you've setup for your OS);
6. Move `RiderSourceCodeAccess` folder from `{UnrealProjectRoot}/Plugins/Developer` to `{UnrealEngineRoot}/Engine/Plugins/Developer`;
7. Start Unreal Editor;
8. Edit > Editor Preferences ... > General > Source Code > Source Code Editor;
9. Select Rider from drop down list. NB: only Rider with C++ plugins will be available from drop down list.
